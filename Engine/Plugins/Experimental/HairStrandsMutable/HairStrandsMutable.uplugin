{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Mutable Groom Extensions",
	"Description": "Adds Mutable functionality to work with Grooms from the HairStrands plugin",
	"Category": "Animation",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "HairStrandsMutable",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "HairStrandsMutableEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "Default"
		}

	],
	"Plugins": [
		{
			"Name": "HairStrands",
			"Enabled": true
		},
		{
			"Name": "Mutable",
			"Enabled": true
		}
	]

}