// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class HairStrandsMutable : ModuleRules
	{
        public HairStrandsMutable(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.Add(ModuleDirectory + "/Private");
			PublicIncludePaths.Add(ModuleDirectory + "/Public");
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"CustomizableObject",
					"HairStrandsCore",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Niagara"
				});
		}
	}
}
