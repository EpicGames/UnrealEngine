// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsMutableExtension.h"

#include "Algo/AnyOf.h"
#include "GroomComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "MuCO/CustomizableObject.h"
#include "MuCO/CustomizableObjectInstance.h"
#include "MuCO/CustomizableObjectInstanceUsage.h"
#include "MuCO/CustomizableSkeletalComponent.h"
#include "MuCO/CustomizableSkeletalMeshActor.h"
#include "UObject/UObjectGlobals.h"

const FName GroomComponentTag(TEXT("Mutable"));

const FName UHairStrandsMutableExtension::GroomPinType(TEXT("Groom"));
const FName UHairStrandsMutableExtension::GroomsBaseNodePinName(TEXT("Grooms"));
const FText UHairStrandsMutableExtension::GroomNodeCategory(FText::FromString(TEXT("Grooms")));

TArray<FCustomizableObjectPinType> UHairStrandsMutableExtension::GetPinTypes() const
{
	TArray<FCustomizableObjectPinType> Result;
	
	FCustomizableObjectPinType& GroomType = Result.AddDefaulted_GetRef();
	GroomType.Name = GroomPinType;
	GroomType.DisplayName = FText::FromString(TEXT("Groom"));
	GroomType.Color = FLinearColor::Red;

	return Result;
}

TArray<FObjectNodeInputPin> UHairStrandsMutableExtension::GetAdditionalObjectNodePins() const
{
	TArray<FObjectNodeInputPin> Result;

	FObjectNodeInputPin& GroomInputPin = Result.AddDefaulted_GetRef();
	GroomInputPin.PinType = GroomPinType;
	GroomInputPin.PinName = GroomsBaseNodePinName;
	GroomInputPin.DisplayName = FText::FromString(TEXT("Groom"));
	GroomInputPin.bIsArray = true;

	return Result;
}

void UHairStrandsMutableExtension::OnCustomizableObjectInstanceUsageUpdated(UCustomizableObjectInstanceUsage& InstanceUsage, TArray<TObjectPtr<const UObject>>& ExtensionData) const
{
	Super::OnCustomizableObjectInstanceUsageUpdated(InstanceUsage, ExtensionData);
	
	UCustomizableObjectInstance* Instance = InstanceUsage.GetCustomizableObjectInstance();
	if (!Instance)
	{
		return;
	}

	USkeletalMeshComponent* AttachedParent = InstanceUsage.GetAttachParent();
	if (!AttachedParent)
	{
		return;
	}
	
	USceneComponent* AttachParent = InstanceUsage.GetAttachParent();
	
	TArray<UGroomComponent*> UpdateGroomComponents; // Grooms that must be present at the end of this update.

	for (const UObject* Extension : ExtensionData)
	{
		const UGroomCompiledData* GroomData = Cast<UGroomCompiledData>(Extension);
		if (!GroomData)
		{
			continue;
		}
		
		if (GroomData->ComponentName != InstanceUsage.GetComponentName())
		{
			continue;
		}
		
		bool bAlreadyExists = false;

		for (USceneComponent* Component : AttachParent->GetAttachChildren())
		{
			UGroomComponent* GroomComponent = Cast<UGroomComponent>(Component);
			if (!GroomComponent)
			{
				continue;
			}

			if (GroomComponent->GroomAsset == GroomData->GroomAsset &&
				GroomComponent->GroomCache == GroomData->GroomCache &&
				GroomComponent->BindingAsset == GroomData->BindingAsset &&
				GroomComponent->PhysicsAsset == GroomData->PhysicsAsset &&
				GroomComponent->AttachmentName == GroomData->AttachmentName &&
				GroomComponent->OverrideMaterials == GroomData->OverrideMaterials)
			{
				UpdateGroomComponents.Add(GroomComponent);
				bAlreadyExists = true;
				break;
			}
		}

		if (bAlreadyExists)
		{
			continue;
		}
		
		UGroomComponent* GroomComponent = NewObject<UGroomComponent>(AttachParent, GroomData->GroomComponentName);
		GroomComponent->GroomAsset = GroomData->GroomAsset;
		GroomComponent->GroomCache = GroomData->GroomCache;
		GroomComponent->BindingAsset = GroomData->BindingAsset;
		GroomComponent->PhysicsAsset = GroomData->PhysicsAsset;
		GroomComponent->AttachmentName = GroomData->AttachmentName;
		GroomComponent->OverrideMaterials = GroomData->OverrideMaterials;
		
		// Work around UE-158069
		GroomComponent->CreationMethod = EComponentCreationMethod::Instance;

		GroomComponent->ComponentTags.Add(GroomComponentTag);
		
		GroomComponent->AttachToComponent(AttachParent, FAttachmentTransformRules::KeepRelativeTransform);
		GroomComponent->PrecachePSOs();
	
		AActor* MyOwner = GroomComponent->GetOwner();
		UWorld* MyOwnerWorld = (MyOwner ? MyOwner->GetWorld() : nullptr);
		if (MyOwnerWorld) // Avoid ensure when registering on Movie Sequence Template actors.
		{
			GroomComponent->RegisterComponent();
		}

		UpdateGroomComponents.Add(GroomComponent);
	}

	// Destroy old Mutable grooms.
	TArray<USceneComponent*> Children;
	AttachParent->GetChildrenComponents(false, Children);
	for (USceneComponent* Child : Children)
	{
		if (Child->IsA(UGroomComponent::StaticClass()) &&
			Child->ComponentTags.Contains(GroomComponentTag) &&
			!UpdateGroomComponents.Contains(Child))
		{
			Child->DestroyComponent();
		}
	}
}


void UHairStrandsMutableExtension::OnCustomizableObjectInstanceUsageDiscarded(UCustomizableObjectInstanceUsage& InstanceUsage) const
{
	Super::OnCustomizableObjectInstanceUsageDiscarded(InstanceUsage);

	USceneComponent* AttachParent = InstanceUsage.GetAttachParent();
	if (!AttachParent)
	{
		return;
	}

	TArray<USceneComponent*> Children;
	AttachParent->GetChildrenComponents(false, Children);
	for (USceneComponent* Child : Children)
	{
		if (Child->IsA(UGroomComponent::StaticClass()) &&
			Child->ComponentTags.Contains(GroomComponentTag))
		{
			Child->DestroyComponent();
		}
	}
}
