// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsMutableModule.h"

#include "MuCO/ICustomizableObjectModule.h"
#include "HairStrandsMutableExtension.h"

class FHairStrandsMutableModule : public IModuleInterface
{
	virtual void StartupModule() override
	{
		Extension = TWeakObjectPtr<const UHairStrandsMutableExtension>(GetDefault<UHairStrandsMutableExtension>());
		ICustomizableObjectModule::Get().RegisterExtension(Extension.Get());
	}

	virtual void ShutdownModule() override
	{
		if (ICustomizableObjectModule::IsAvailable()
			&& Extension.IsValid())
		{
			ICustomizableObjectModule::Get().UnregisterExtension(Extension.Get());
		}
	}

private:
	TWeakObjectPtr<const UHairStrandsMutableExtension> Extension;
};

IMPLEMENT_MODULE(FHairStrandsMutableModule, HairStrandsMutable);
