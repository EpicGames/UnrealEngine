// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MuCO/CustomizableObjectExtension.h"
#include "MuR/ExtensionData.h"

#include "HairStrandsMutableExtension.generated.h"

class UGroomAsset;
class UGroomCache;
class UGroomBindingAsset;
class UPhysicsAsset;
class UMaterialInterface;

/** Used as ExtensionData to represent a Groom Asset in a Customizable Object graph */
USTRUCT()
struct HAIRSTRANDSMUTABLE_API FGroomPinData
{
	GENERATED_BODY()

public:
	/** Name of the mesh component this groom will be attached to */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FName ComponentName;

	/** The groom asset to use. See UGroomComponent::GroomAsset. */
	UPROPERTY(EditAnywhere, Category = Groom)
	TObjectPtr<UGroomAsset> GroomAsset;

	/** See UGroomComponent::GroomCache. */
	UPROPERTY(EditAnywhere, Category = GroomCache, meta = (EditCondition = "BindingAsset == nullptr"))
	TObjectPtr<UGroomCache> GroomCache;

	/** See UGroomComponent::BindingAsset. */
	UPROPERTY(EditAnywhere, Category = Groom, meta = (EditCondition = "GroomCache == nullptr"))
	TObjectPtr<UGroomBindingAsset> BindingAsset;

	/** See UGroomComponent::PhysicsAsset. */
	UPROPERTY(EditAnywhere, Category = Simulation)
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	/** See UGroomComponent::AttachmentName. */
	UPROPERTY(EditAnywhere, Category = Groom)
	FString AttachmentName;

	/** Name of the created groom component */
	UPROPERTY(EditAnywhere, Category = Groom)
	FName GroomComponentName;

	/** See UMeshComponent::OverrideMaterials. */
	UPROPERTY(EditAnywhere, Category = Rendering)
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;
};

UCLASS()
class UGroomCompiledData : public UObject
{
	GENERATED_BODY()
	
public:
	/** Name of the mesh component this groom will be attached to */
	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FName ComponentName;

	/** The groom asset to use. See UGroomComponent::GroomAsset. */
	UPROPERTY(EditAnywhere, Category = Groom)
	TObjectPtr<UGroomAsset> GroomAsset;

	/** See UGroomComponent::GroomCache. */
	UPROPERTY(EditAnywhere, Category = GroomCache, meta = (EditCondition = "BindingAsset == nullptr"))
	TObjectPtr<UGroomCache> GroomCache;

	/** See UGroomComponent::BindingAsset. */
	UPROPERTY(EditAnywhere, Category = Groom, meta = (EditCondition = "GroomCache == nullptr"))
	TObjectPtr<UGroomBindingAsset> BindingAsset;

	/** See UGroomComponent::PhysicsAsset. */
	UPROPERTY(EditAnywhere, Category = Simulation)
	TObjectPtr<UPhysicsAsset> PhysicsAsset;

	/** See UGroomComponent::AttachmentName. */
	UPROPERTY(EditAnywhere, Category = Groom)
	FString AttachmentName;

	/** Name of the created groom component */
	UPROPERTY(EditAnywhere, Category = Groom)
	FName GroomComponentName;

	/** See UMeshComponent::OverrideMaterials. */
	UPROPERTY(EditAnywhere, Category = Rendering)
	TArray<TObjectPtr<UMaterialInterface>> OverrideMaterials;
};

/**
 * An extension for Mutable that allows users to bring Grooms from the HairStrands plugin into
 * their Customizable Objects
 */
UCLASS(MinimalAPI)
class UHairStrandsMutableExtension : public UCustomizableObjectExtension
{
	GENERATED_BODY()

public:
	/** UCustomizableObjectExtension interface */
	virtual TArray<FCustomizableObjectPinType> GetPinTypes() const override;
	virtual TArray<FObjectNodeInputPin> GetAdditionalObjectNodePins() const override;
	virtual void OnCustomizableObjectInstanceUsageUpdated(UCustomizableObjectInstanceUsage& Usage, TArray<TObjectPtr<const UObject>>& ExtensionData) const override;
	virtual void OnCustomizableObjectInstanceUsageDiscarded(UCustomizableObjectInstanceUsage& Usage) const override;

	HAIRSTRANDSMUTABLE_API static const FName GroomPinType;
	HAIRSTRANDSMUTABLE_API static const FName GroomsBaseNodePinName;
	HAIRSTRANDSMUTABLE_API static const FText GroomNodeCategory;
};
