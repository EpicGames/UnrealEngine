// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class HairStrandsMutableEditor : ModuleRules
	{
        public HairStrandsMutableEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateIncludePaths.Add(ModuleDirectory + "/Private");
			PublicIncludePaths.Add(ModuleDirectory + "/Public");
			
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"CustomizableObjectEditor",
					"MutableTools",
					"MutableRuntime"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"HairStrandsMutable",
					"HairStrandsCore",
				});
		}
	}
}
