// Copyright Epic Games, Inc. All Rights Reserved.

#include "CustomizableObjectNodeGroomConstant.h"

#include "MuCOE/EdGraphSchema_CustomizableObject.h"
#include "MuCOE/RemapPins/CustomizableObjectNodeRemapPins.h"
#include "MuT/NodeExtensionDataConstant.h"
#include "HairStrandsMutableExtension.h"
#include "MuCOE/ExtensionDataCompilerInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(CustomizableObjectNodeGroomConstant)

#define LOCTEXT_NAMESPACE "HairStrandsMutableEditor"


UCustomizableObjectNodeGroomConstant::UCustomizableObjectNodeGroomConstant()
{
	CompiledData = CreateDefaultSubobject<UGroomCompiledData>(FName("GroomData"));
}


FText UCustomizableObjectNodeGroomConstant::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return LOCTEXT("Groom_Constant", "Groom Constant");
}


FLinearColor UCustomizableObjectNodeGroomConstant::GetNodeTitleColor() const
{
	const UEdGraphSchema_CustomizableObject* Schema = GetDefault<UEdGraphSchema_CustomizableObject>();
	return Schema->GetPinTypeColor(UHairStrandsMutableExtension::GroomPinType);
}

FText UCustomizableObjectNodeGroomConstant::GetTooltipText() const
{
	return LOCTEXT("Groom_Constant_Tooltip", "Imports a Groom");
}


void UCustomizableObjectNodeGroomConstant::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	
	if (PropertyChangedEvent.MemberProperty)
	{
		CopyCompiledData();
	}
}

void UCustomizableObjectNodeGroomConstant::AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins)
{
	CustomCreatePin(EGPD_Output, UHairStrandsMutableExtension::GroomPinType, UHairStrandsMutableExtension::GroomsBaseNodePinName, UHairStrandsMutableExtension::GroomNodeCategory);
}

bool UCustomizableObjectNodeGroomConstant::ShouldAddToContextMenu(FText& OutCategory) const
{
	OutCategory = UEdGraphSchema_CustomizableObject::NC_Experimental;
	return true;
}

bool UCustomizableObjectNodeGroomConstant::IsExperimental() const
{
	return true;
}

UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionData> UCustomizableObjectNodeGroomConstant::GenerateMutableNode(FExtensionDataCompilerInterface& CompilerInterface) const
{
	check(IsInGameThread());
	check(CompiledData);

	UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionDataConstant> Result = new UE::Mutable::Private::NodeExtensionDataConstant();
	Result->ExtensionDataId = CompilerInterface.MakeExtensionData(*CompiledData, true);

	return Result;
}

void UCustomizableObjectNodeGroomConstant::CopyCompiledData()
{
	check(CompiledData);
	CompiledData->ComponentName = GroomData.ComponentName;
	CompiledData->GroomAsset = GroomData.GroomAsset;
	CompiledData->GroomCache = GroomData.GroomCache;
	CompiledData->BindingAsset = GroomData.BindingAsset;
	CompiledData->PhysicsAsset = GroomData.PhysicsAsset;
	CompiledData->AttachmentName = GroomData.AttachmentName;
	CompiledData->GroomComponentName = GroomData.GroomComponentName;
	CompiledData->OverrideMaterials = GroomData.OverrideMaterials;
}

#undef LOCTEXT_NAMESPACE
