// Copyright Epic Games, Inc. All Rights Reserved.

#include "HairStrandsMutableEditorModule.h"

IMPLEMENT_MODULE(FDefaultModuleImpl, HairStrandsMutableEditor);
