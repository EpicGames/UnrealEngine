// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HairStrandsMutableExtension.h"
#include "MuCOE/ICustomizableObjectExtensionNode.h"
#include "MuCOE/Nodes/CustomizableObjectNodeExtensionDataConstant.h"

#include "CustomizableObjectNodeGroomConstant.generated.h"

/** Imports a Groom into the Customizable Object graph */
UCLASS()
class UCustomizableObjectNodeGroomConstant : public UCustomizableObjectNodeExtensionDataConstant, public ICustomizableObjectExtensionNode
{
	GENERATED_BODY()

public:
	UCustomizableObjectNodeGroomConstant();
	
	/** EdGraphNode interface */
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FText GetTooltipText() const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	/** UCustomizableObjectNode interface */
	virtual void AllocateDefaultPins(UCustomizableObjectNodeRemapPins* RemapPins) override;
	virtual bool ShouldAddToContextMenu(FText& OutCategory) const override;
	virtual bool IsExperimental() const override;
	
	/** ICustomizableObjectExtensionNode interface */
	virtual UE::Mutable::Private::Ptr<UE::Mutable::Private::NodeExtensionData> GenerateMutableNode(FExtensionDataCompilerInterface& CompilerInterface) const override;

private:
	/** Own interface */
	void CopyCompiledData();

	UPROPERTY(EditAnywhere, Category = CustomizableObject)
	FGroomPinData GroomData;
	
	UPROPERTY()
	TObjectPtr<UGroomCompiledData> CompiledData;
	
};

